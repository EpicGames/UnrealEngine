{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Plugin Utilities",
	"Description": "Helpers to create and edit plugins. Used by Plugin Browser.",
	"Category": "Programming",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PluginUtils",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "PythonScriptPlugin",
			"Enabled": false,
			"Optional": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}
