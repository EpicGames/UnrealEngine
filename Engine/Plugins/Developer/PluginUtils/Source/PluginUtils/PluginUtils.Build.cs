// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PluginUtils : ModuleRules
	{
		public PluginUtils(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Projects",
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetRegistry",
					"AssetTools",
					"CoreUObject",
					"DesktopPlatform",
					"EditorScriptingUtilities",
					"Engine",
					"GameProjectGeneration",
					"GameplayTags",
					"SourceControl",
					"SubobjectDataInterface",
					"UnrealEd"
				}
			);
			
			if (Target.bBuildEditor && Target.bCompilePython)
			{
				PrivateDependencyModuleNames.Add("PythonScriptPlugin");
				PublicDefinitions.Add("COMPILE_PYTHON=1");
			}
			else
			{
				PublicDefinitions.Add("COMPILE_PYTHON=0");
			}
		}
	}
}
