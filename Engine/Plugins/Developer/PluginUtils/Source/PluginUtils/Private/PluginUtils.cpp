// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PluginUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "DesktopPlatformModule.h"
#include "EditorAssetLibrary.h"
#include "GameProjectUtils.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "HAL/PlatformFileManager.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IDesktopPlatform.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "PackageTools.h"
#include "SourceControlHelpers.h"
#include "SubobjectDataSubsystem.h"
#include "UnrealEdMisc.h"
#include "Misc/CommandLine.h"
#include "UObject/CoreRedirects.h"
#include "UObject/PropertyAccessUtil.h"
#include "UObject/UObjectIterator.h"

#if COMPILE_PYTHON
#include "IPythonScriptPlugin.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogPluginUtils, Log, All);

#define LOCTEXT_NAMESPACE "PluginUtils"

namespace PluginUtils
{
	// The text macro to replace with the actual plugin name when copying files
	const FString PLUGIN_NAME = TEXT("PLUGIN_NAME");

	FString ConstructPackageName(const FString& RootPath, const TCHAR* Filename)
	{
		FString PackageName;

		const FString ContentPath = FPaths::Combine(RootPath, TEXT("Content"));
		if (FPaths::IsUnderDirectory(Filename, ContentPath))
		{
			FStringView RelativePkgPath;
			if (FPathViews::TryMakeChildPathRelativeTo(Filename, ContentPath, RelativePkgPath))
			{
				RelativePkgPath = FPathViews::GetBaseFilenameWithPath(RelativePkgPath);
				if (RelativePkgPath.Len() > 0 && !RelativePkgPath.EndsWith(TEXT("/")))
				{
					PackageName = FPaths::Combine(TEXT("/"), FPaths::GetBaseFilename(RootPath), RelativePkgPath);
				}
			}
		}
		return PackageName;
	}

	bool CopyPluginTemplateFolder(const TCHAR* DestinationDirectory, const TCHAR* Source, const FString& PluginName, const FString& NameToReplace, TArray<FCoreRedirect>& InOutCoreRedirectList, TArray<FString>& InOutFilePathsWritten, FText* OutFailReason = nullptr)
	{
		check(DestinationDirectory);
		check(Source);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		FString DestDir(DestinationDirectory);
		FPaths::NormalizeDirectoryName(DestDir);

		FString SourceDir(Source);
		FPaths::NormalizeDirectoryName(SourceDir);

		// Does Source dir exist?
		if (!PlatformFile.DirectoryExists(*SourceDir))
		{
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("InvalidPluginTemplateFolder", "Plugin template folder doesn't exist\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(SourceDir)));
			}
			return false;
		}

		// Destination directory exists already or can be created ?
		if (!PlatformFile.DirectoryExists(*DestDir) && !PlatformFile.CreateDirectoryTree(*DestDir))
		{
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailedToCreateDestinationFolder", "Failed to create destination folder\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(DestDir)));
			}
			return false;
		}

		// Copy all files and directories, renaming specific sections to the plugin name
		struct FCopyPluginFilesAndDirs : public IPlatformFile::FDirectoryVisitor
		{
			IPlatformFile& PlatformFile;
			const TCHAR* SourceRoot;
			const TCHAR* DestRoot;
			const FString& PluginName;
			const FString& NameToReplace;
			TArray<FString> NameReplacementFileTypes;
			TArray<FString> IgnoredFileTypes;
			TArray<FString> CopyUnmodifiedFileTypes;
			TArray<FString>& FilePathsWritten;
			FText* FailReason;
			TArray<FCoreRedirect>& CoreRedirectList;

			FCopyPluginFilesAndDirs(IPlatformFile& InPlatformFile, const TCHAR* InSourceRoot, const TCHAR* InDestRoot, const FString& InPluginName, const FString& InNameToReplace, TArray<FString>& InFilePathsWritten, TArray<FCoreRedirect>& CoreRedirectList, FText* InFailReason)
				: PlatformFile(InPlatformFile)
				, SourceRoot(InSourceRoot)
				, DestRoot(InDestRoot)
				, PluginName(InPluginName)
				, NameToReplace(InNameToReplace)
				, FilePathsWritten(InFilePathsWritten)
				, FailReason(InFailReason)
				, CoreRedirectList(CoreRedirectList)
			{
				// Which file types we want to replace instances of PLUGIN_NAME with the new Plugin Name
				NameReplacementFileTypes.Add(TEXT("cs"));
				NameReplacementFileTypes.Add(TEXT("cpp"));
				NameReplacementFileTypes.Add(TEXT("h"));
				NameReplacementFileTypes.Add(TEXT("vcxproj"));
				NameReplacementFileTypes.Add(TEXT("uplugin"));

				// Which file types do we want to ignore
				IgnoredFileTypes.Add(TEXT("opensdf"));
				IgnoredFileTypes.Add(TEXT("sdf"));
				IgnoredFileTypes.Add(TEXT("user"));
				IgnoredFileTypes.Add(TEXT("suo"));

				// Which file types do we want to copy completely unmodified
				CopyUnmodifiedFileTypes.Add(TEXT("uasset"));
				CopyUnmodifiedFileTypes.Add(TEXT("umap"));
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
			{
				FString NewName(FilenameOrDirectory);
				// change the root and rename paths/files
				NewName.RemoveFromStart(SourceRoot);
				NewName = NewName.Replace(*PLUGIN_NAME, *NameToReplace, ESearchCase::CaseSensitive);
				NewName = FPaths::Combine(DestRoot, *NewName);

				if (bIsDirectory)
				{
					// create new directory structure
					if (!PlatformFile.CreateDirectoryTree(*NewName) && !PlatformFile.DirectoryExists(*NewName))
					{
						if (FailReason)
						{
							*FailReason = FText::Format(LOCTEXT("FailedToCreatePluginSubFolder", "Failed to create plugin subfolder\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(NewName)));
						}
						return false;
					}
				}
				else
				{
					FString NewExt = FPaths::GetExtension(FilenameOrDirectory);

					if (!IgnoredFileTypes.Contains(NewExt))
					{
						if (CopyUnmodifiedFileTypes.Contains(NewExt))
						{
							// Copy unmodified files with the original name, but do rename the directories
							FString CleanFilename = FPaths::GetCleanFilename(FilenameOrDirectory);
							FString CopyToPath = FPaths::GetPath(NewName);

							NewName = FPaths::Combine(CopyToPath, CleanFilename);

							// Redirect the template package name to the generated package name to fix up internal references.
							FString SourcePackageName = ConstructPackageName(SourceRoot, FilenameOrDirectory);
							FString DestPackageName = ConstructPackageName(DestRoot, *NewName);
							CoreRedirectList.Add(FCoreRedirect(ECoreRedirectFlags::Type_Package, SourcePackageName, DestPackageName));
						}

						if (PlatformFile.FileExists(*NewName))
						{
							// Delete destination file if it exists
							PlatformFile.DeleteFile(*NewName);
						}

						// If file of specified extension - open the file as text and replace PLUGIN_NAME in there before saving
						const bool bIsTagsIni = NewExt.Equals(TEXT("ini")) && NewName.Contains(TEXT("/Config/Tags/"));
						if (NameReplacementFileTypes.Contains(NewExt) || bIsTagsIni)
						{
							FString OutFileContents;
							if (!FFileHelper::LoadFileToString(OutFileContents, FilenameOrDirectory))
							{
								if (FailReason)
								{
									*FailReason = FText::Format(LOCTEXT("FailedToReadPluginTemplateFile", "Failed to read plugin template file\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(FilenameOrDirectory)));
								}
								return false;
							}

							OutFileContents = OutFileContents.Replace(*PLUGIN_NAME, *NameToReplace, ESearchCase::CaseSensitive);

							// For some content, we also want to export a PLUGIN_NAME_API text macro, which requires that the plugin name
							// be all capitalized

							FString ReplaceNameAPI = NameToReplace + TEXT("_API");

							OutFileContents = OutFileContents.Replace(*ReplaceNameAPI, *ReplaceNameAPI.ToUpper(), ESearchCase::CaseSensitive);

							// Special case the .uplugin as we may be copying a real plugin and the filename will not be named PLUGIN_NAME
							// as it needs to be loaded by the editor. Ensure the new plugin name is used.
							if (NewExt == TEXT("uplugin"))
							{
								FString CopyToPath = FPaths::GetPath(NewName);
								NewName = FPaths::Combine(CopyToPath, PluginName) + TEXT(".uplugin");
							}

							if (!FFileHelper::SaveStringToFile(OutFileContents, *NewName))
							{
								if (FailReason)
								{
									*FailReason = FText::Format(LOCTEXT("FailedToWritePluginFile", "Failed to write plugin file\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(NewName)));
								}
								return false;
							}
						}
						else
						{
							// Copy file from source
							if (!PlatformFile.CopyFile(*NewName, FilenameOrDirectory))
							{
								// Not all files could be copied
								if (FailReason)
								{
									*FailReason = FText::Format(LOCTEXT("FailedToCopyPluginTemplateFile", "Failed to copy plugin template file\nFrom: {0}\nTo: {1}"), FText::FromString(FPaths::ConvertRelativePathToFull(FilenameOrDirectory)), FText::FromString(FPaths::ConvertRelativePathToFull(NewName)));
								}
								return false;
							}
						}
						FilePathsWritten.Add(NewName);
					}
				}
				return true; // continue searching
			}
		};

		// copy plugin files and directories visitor
		FCopyPluginFilesAndDirs CopyFilesAndDirs(PlatformFile, *SourceDir, *DestDir, PluginName, NameToReplace, InOutFilePathsWritten, InOutCoreRedirectList, OutFailReason);

		// create all files subdirectories and files in subdirectories!
		return PlatformFile.IterateDirectoryRecursively(*SourceDir, CopyFilesAndDirs);
	}

	void FixupPluginTemplateAssets(const FString& PluginName, const FString& NameToReplace, TMap<FString, FString>& OutModifiedAssetPaths)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);

		struct FFixupPluginAssets : public IPlatformFile::FDirectoryVisitor
		{
			IPlatformFile& PlatformFile;
			const FString& NameToReplace;
			const FString& PluginBaseDir;

			TArray<FString> FilesToScan;

			FFixupPluginAssets(IPlatformFile& InPlatformFile, const FString& InNameToReplace, const FString& InPluginBaseDir)
				: PlatformFile(InPlatformFile)
				, NameToReplace(InNameToReplace)
				, PluginBaseDir(InPluginBaseDir)
			{
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if (!bIsDirectory)
				{
					FString Extension = FPaths::GetExtension(FilenameOrDirectory);

					// Only interested in fixing up uassets and umaps...anything else we leave alone
					if (Extension == TEXT("uasset") || Extension == TEXT("umap"))
					{
						FilesToScan.Add(FilenameOrDirectory);
					}
				}

				return true;
			}

			void ReplaceGameplayTagsInObject(UObject* Object)
			{
				if (!Object)
				{
					return;
				}

				for (TPropertyValueIterator<FProperty> It(Object->GetClass(), Object, EPropertyValueIteratorFlags::FullRecursion); It; ++It)
				{
					if (const FStructProperty* StructProperty = CastField<FStructProperty>(It.Key()))
					{
						if (UScriptStruct* StructType = StructProperty->Struct)
						{
							if (StructType->IsChildOf(FGameplayTag::StaticStruct()))
							{
								const void* PropertyValue = It.Value();

								const FGameplayTag* GameplayTag = reinterpret_cast<const FGameplayTag*>(PropertyValue);
								if (GameplayTag && GameplayTag->IsValid() && GameplayTag->ToString().Contains(PLUGIN_NAME))
								{
									FGameplayTag NewTag = UGameplayTagsManager::Get().RequestGameplayTag(FName(GameplayTag->ToString().Replace(*PLUGIN_NAME, *NameToReplace, ESearchCase::CaseSensitive)));
									It.Key()->CopyCompleteValue(const_cast<void*>(PropertyValue), &NewTag);
								}
							}
						}
					}
				}
			}

			/**
			 * Fixes up any assets that contain the PLUGIN_NAME text macro, since those need to be renamed by the engine for the change to
			 * stick (as opposed to just renaming the file) as well as regenerating the localization ids for the assets in the plugin.
			 */
			void PerformFixup(TMap<FString, FString>& OutModifiedAssetPaths)
			{
				TArray<FAssetRenameData> AssetRenameData;

				if (FilesToScan.Num() > 0)
				{
					IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
					AssetRegistry.ScanFilesSynchronous(FilesToScan);

					TMap<TWeakObjectPtr<UObject>, FString> AssetToOriginalFilePathMap;
					for (const FString& File : FilesToScan)
					{
						TArray<FAssetData> Assets;

						FString PackageName;
						if (FPackageName::TryConvertFilenameToLongPackageName(File, PackageName))
						{
							AssetRegistry.GetAssetsByPackageName(*PackageName, Assets);
						}

						for (FAssetData AssetData : Assets)
						{
							UObject* LoadedAsset = AssetData.GetAsset();
							const UClass* LoadedClass = AssetData.GetClass(EResolveClass::Yes);
							if (LoadedClass == UBlueprint::StaticClass())
							{
								if (UClass* BPClass = UEditorAssetLibrary::LoadBlueprintClass(AssetData.GetPackage()->GetPathName()))
								{
									LoadedAsset = BPClass->GetDefaultObject();

									USubobjectDataSubsystem* DataSubsystem = USubobjectDataSubsystem::Get();

									if (ensure(DataSubsystem))
									{
										TArray<FSubobjectDataHandle> SubobjectData;
										DataSubsystem->GatherSubobjectData(LoadedAsset, SubobjectData);

										for (const FSubobjectDataHandle& Handle : SubobjectData)
										{
											if (UObject* SubObject = const_cast<UObject*>(Handle.GetData()->GetObjectForBlueprint(Cast<UBlueprint>(LoadedAsset))))
											{
												ReplaceGameplayTagsInObject(SubObject);
											}
										}
									}
								}
							}

							ReplaceGameplayTagsInObject(LoadedAsset);

							const FString AssetName = AssetData.AssetName.ToString().Replace(*PLUGIN_NAME, *NameToReplace, ESearchCase::CaseSensitive);
							const FString AssetPath = AssetData.PackagePath.ToString().Replace(*PLUGIN_NAME, *NameToReplace, ESearchCase::CaseSensitive);

							// When the new name and old name are an exact match the on disk asset will be missing. This is because the clean up step of RenameAssetsWithDialog deletes the old file and no new file will be created. 
							// This behaviour is not supported by the existing editor workflows. Rather than change the functionality, the rename step can instead be skipped.
							if (AssetName != AssetData.AssetName)
							{
								AssetToOriginalFilePathMap.Add(AssetData.GetAsset(), File);
								FAssetRenameData RenameData(AssetData.GetAsset(), AssetPath, AssetName);

								AssetRenameData.Add(RenameData);
							}

#if USE_STABLE_LOCALIZATION_KEYS
							// ANY assets created/duplicated from a plugin template should have its package localization ID reset to avoid conflicts.
							if (UObject* Asset = AssetData.GetAsset())
							{
								Asset->Modify();
								TextNamespaceUtil::ClearPackageNamespace(Asset);
								TextNamespaceUtil::EnsurePackageNamespace(Asset);
							}
#endif // USE_STABLE_LOCALIZATION_KEYS
						}
					}

					if (AssetRenameData.Num() > 0)
					{
						FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
						AssetToolsModule.Get().RenameAssetsWithDialog(AssetRenameData);

						for(const FAssetRenameData& RenamedAsset : AssetRenameData)
						{
							if (RenamedAsset.Asset.IsValid())
							{
								if (FString* OldPath = AssetToOriginalFilePathMap.Find(RenamedAsset.Asset))
								{
									if (const UPackage* Package = RenamedAsset.Asset->GetPackage())
									{
										FString PackageFilename;
										if (FPackageName::TryConvertLongPackageNameToFilename(Package->GetName(), PackageFilename, Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension()))
										{
											FString FullOldPath = FPaths::ConvertRelativePathToFull(*OldPath);
											FPaths::RemoveDuplicateSlashes(FullOldPath);
											OutModifiedAssetPaths.Add(FullOldPath, PackageFilename);
										}
									}
								}
							}
						}
					}
				}
			}
		};

		if (Plugin.IsValid())
		{
			UGameplayTagsManager::Get().AddTagIniSearchPath(Plugin->GetBaseDir() / TEXT("Config") / TEXT("Tags"));
			UGameplayTagsManager::Get().EditorRefreshGameplayTagTree();

			const FString PluginBaseDir = Plugin->GetBaseDir();
			FFixupPluginAssets FixupPluginAssets(PlatformFile, NameToReplace, PluginBaseDir);
			PlatformFile.IterateDirectoryRecursively(*PluginBaseDir, FixupPluginAssets);
			FixupPluginAssets.PerformFixup(OutModifiedAssetPaths);
		}
	}

	TSharedPtr<IPlugin> LoadPluginInternal(const FString& PluginName, const FString& PluginLocation, const FString& PluginFilePath, FPluginUtils::FLoadPluginParams& LoadParams, const bool bIsNewPlugin)
	{
		LoadParams.bOutAlreadyLoaded = false;

		if (LoadParams.bUpdateProjectPluginSearchPath && !PluginLocation.IsEmpty())
		{
			FPluginUtils::AddToPluginSearchPathIfNeeded(PluginLocation, /*bRefreshPlugins=*/false, /*bUpdateProjectFile=*/true);
			IPluginManager::Get().RefreshPluginsList();
		}
		else
		{
			if (!IPluginManager::Get().AddToPluginsList(PluginFilePath, LoadParams.OutFailReason))
			{
				return nullptr;
			}
		}

		// Find the plugin in the manager.
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
		if (!Plugin)
		{
			if (LoadParams.OutFailReason)
			{
				*LoadParams.OutFailReason = FText::Format(LOCTEXT("FailedToRegisterPlugin", "Failed to register plugin\n{0}"), FText::FromString(PluginFilePath));
			}
			return nullptr;
		}

		// Double check the path matches
		if (!FPaths::IsSamePath(Plugin->GetDescriptorFileName(), PluginFilePath))
		{
			if (LoadParams.OutFailReason)
			{
				const FString PluginFilePathFull = FPaths::ConvertRelativePathToFull(Plugin->GetDescriptorFileName());
				*LoadParams.OutFailReason = FText::Format(LOCTEXT("PluginNameAlreadyUsed", "There's already a plugin named {0} at this location:\n{1}"), FText::FromString(PluginName), FText::FromString(PluginFilePathFull));
			}
			return nullptr;
		}

		const FString PluginRootFolder = Plugin->CanContainContent() ? Plugin->GetMountedAssetPath() : FString();

		LoadParams.bOutAlreadyLoaded = !bIsNewPlugin && Plugin->IsEnabled() && (PluginRootFolder.IsEmpty() || FPackageName::MountPointExists(PluginRootFolder));

		// Enable this plugin in the project
		if (LoadParams.bEnablePluginInProject)
		{
			FText FailReason;
			if (!IProjectManager::Get().SetPluginEnabled(PluginName, true, FailReason))
			{
				if (LoadParams.OutFailReason)
				{
					*LoadParams.OutFailReason = FText::Format(LOCTEXT("FailedToEnablePluginInProject", "Failed to enable plugin in current project\n{0}"), FailReason);
				}
				return nullptr;
			}
		}

		if (!LoadParams.bOutAlreadyLoaded)
		{
			// Mount the new plugin (mount content folder if any and load modules if any)
			if (bIsNewPlugin)
			{
				IPluginManager::Get().MountNewlyCreatedPlugin(PluginName);
			}
			else
			{
				IPluginManager::Get().MountExplicitlyLoadedPlugin(PluginName);
			}

			if (!Plugin->IsEnabled())
			{
				if (LoadParams.OutFailReason)
				{
					*LoadParams.OutFailReason = FText::Format(LOCTEXT("FailedToEnablePlugin", "Failed to enable plugin because it is not configured as bExplicitlyLoaded=true\n{0}"), FText::FromString(PluginFilePath));
				}
				return nullptr;
			}
		}

		const bool bSelectInContentBrowser = LoadParams.bSelectInContentBrowser && !IsRunningCommandlet();

		if ((bSelectInContentBrowser || LoadParams.bSynchronousAssetsScan) && !PluginRootFolder.IsEmpty() && (LoadParams.bOutAlreadyLoaded || FPackageName::MountPointExists(PluginRootFolder)))
		{
			if (LoadParams.bSynchronousAssetsScan)
			{
				// Scan plugin assets
				IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
				AssetRegistry.ScanPathsSynchronous({ PluginRootFolder }, /*bForceRescan=*/ true);
			}

			if (bSelectInContentBrowser)
			{
				// Select plugin root folder in content browser
				IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
				const bool bIsEnginePlugin = FPaths::IsUnderDirectory(PluginFilePath, FPaths::EnginePluginsDir());
				ContentBrowser.ForceShowPluginContent(bIsEnginePlugin);
				ContentBrowser.SetSelectedPaths({ PluginRootFolder }, /*bNeedsRefresh*/ true);
			}
		}

		return Plugin;
	}

	void InitializeNewPluginParamsWithDescriptor(const FString& PluginName, const FPluginUtils::FNewPluginParams& CreationParams, FPluginUtils::FNewPluginParamsWithDescriptor& ExCreationParams)
	{
		ExCreationParams.PluginIconPath = CreationParams.PluginIconPath;
		ExCreationParams.TemplateFolders = CreationParams.TemplateFolders;
		
		ExCreationParams.PostCreatePythonScriptPath            = CreationParams.PostCreatePythonScriptPath;
		ExCreationParams.PostCreatePythonScriptArguments       = CreationParams.PostCreatePythonScriptArguments;
		ExCreationParams.PostCreatePythonScriptAdditionalArguments = CreationParams.PostCreatePythonScriptAdditionalArguments;

		ExCreationParams.Descriptor.FriendlyName = CreationParams.FriendlyName.Len() > 0 ? CreationParams.FriendlyName : PluginName;
		ExCreationParams.Descriptor.Version = 1;
		ExCreationParams.Descriptor.VersionName = TEXT("1.0");
		ExCreationParams.Descriptor.Category = TEXT("Other");
		ExCreationParams.Descriptor.CreatedBy = CreationParams.CreatedBy;
		ExCreationParams.Descriptor.CreatedByURL = CreationParams.CreatedByURL;
		ExCreationParams.Descriptor.Description = CreationParams.Description;
		ExCreationParams.Descriptor.bIsBetaVersion = CreationParams.bIsBetaVersion;
		ExCreationParams.Descriptor.bCanContainContent = CreationParams.bCanContainContent;
		ExCreationParams.Descriptor.bCanContainVerse = CreationParams.bCanContainVerse;
		ExCreationParams.Descriptor.EnabledByDefault = CreationParams.EnabledByDefault;
		ExCreationParams.Descriptor.bExplicitlyLoaded = CreationParams.bExplicitelyLoaded;
		ExCreationParams.Descriptor.VersePath = CreationParams.VersePath;
		ExCreationParams.Descriptor.VerseVersion = CreationParams.VerseVersion;
		ExCreationParams.Descriptor.bEnableSceneGraph = CreationParams.bEnableSceneGraph;
		ExCreationParams.Descriptor.bEnableSceneGraphAnimation = CreationParams.bEnableSceneGraphAnimation;
		ExCreationParams.Descriptor.bEnableVerseAssetReflection = CreationParams.bEnableVerseAssetReflection;
		ExCreationParams.Descriptor.bEnableIAD = CreationParams.bEnableIAD;

		if (CreationParams.bHasModules)
		{
			ExCreationParams.Descriptor.Modules.Add(FModuleDescriptor(*PluginName, CreationParams.ModuleDescriptorType, CreationParams.LoadingPhase));
		}
	}
	
	FString GetTemplatePluginNames(const TArray<FString> TemplateFolders)
    {
    	FString PluginNames;
    	for (const FString& TemplateFolder : TemplateFolders)
    	{
        	TArray<FString> UPluginFiles;
        	IFileManager::Get().FindFiles(UPluginFiles, *TemplateFolder, TEXT("*.uplugin"));
    
    		for (const FString& UPluginFile : UPluginFiles)
    		{
				if (!PluginNames.IsEmpty())
				{
					PluginNames += ",";
				}
				// Strip the .uplugin extension from each file
                PluginNames += FPaths::GetBaseFilename(UPluginFile);
			}
    	}
        
        return PluginNames;
    }
	
	bool RunPostCreatePythonScript(const FString& PluginName, const FString& PluginFolder, const TArray<FString> TemplateFolders, const FString& ScriptPath, const FString& TemplateArgs, const FString& AdditionalArgs, const bool IsEnginePlugin, FText& OutFailReason)
	{
#if COMPILE_PYTHON
		const TSharedPtr<IPlugin> PythonPlugin = IPluginManager::Get().FindPlugin(TEXT("PythonScriptPlugin"));
		if (!PythonPlugin.IsValid() || !PythonPlugin->IsEnabled())
		{
			OutFailReason = LOCTEXT("PostCreatePythonNotAvailable", "PythonScriptPlugin not loaded and enabled. Post Create Python Script cannot run without it.");
			return false;
		}
		
		IPythonScriptPlugin* PythonScriptPlugin = IPythonScriptPlugin::Get();
		if (!PythonScriptPlugin)
		{
			OutFailReason = LOCTEXT("PostCreatePythonModuleNotAvailable", "PythonScriptPlugin module not found. Post Create Python Script cannot run without it.");
			return false;
		}
		
		if (!PythonScriptPlugin->IsPythonAvailable())
		{
			OutFailReason = LOCTEXT("PostCreatePythonNotAvailableOnTarget", "Engine does not have Python support enabled. Python is required to run Post Create python scripts.");
			return false;
		}
		
		if (!PythonScriptPlugin->IsPythonInitialized())
		{
			OutFailReason = LOCTEXT("PostCreatePythonNotInitialized", "PythonScriptPlugin loaded and enabled but not initialized. Wait until it is initialized and try again.");
			return false;
		}
		
		const FString AbsScriptPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ScriptPath);
		if (!FPaths::FileExists(AbsScriptPath))
		{
			OutFailReason = FText::Format(LOCTEXT("PostCreateScriptNotFound", "Post Create Python Script not found in: {0}"), FText::FromString(AbsScriptPath));
			return false;
		}
		
		const FString PluginRoot = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*PluginFolder);
		if (!FPaths::DirectoryExists(PluginRoot))
		{
			OutFailReason = FText::Format(LOCTEXT("PostCreatePluginNotFound", "Plugin {0} not found in: {1}"), FText::FromString(PluginName), FText::FromString(PluginRoot));
			return false;
		}
		
		auto ValidateArgsBlob = [](const FString& In, FText& OutError) -> bool
			{
				if (In.IsEmpty())
				{
					return true; // empty strings are valid
				}
				
				// Unbalanced quotes
				bool bInQuotes = false;
				int32 BackslashCount = 0;
				for (int32 i = 0; i < In.Len(); ++i)
				{
					const TCHAR Char = In[i];
					
					// Invalid characters
					if (Char == '\0' || Char == '\r' || Char == '\n')
					{
						OutError = FText::Format(LOCTEXT("PostCreatePythonForbiddenCharsInArg", "Arguments passed in (\"{0}\") contain forbidden control characters."), FText::FromString(In));
						return false;
					}
					
					if (Char == '\\')
					{
						BackslashCount++;
						continue;
					}
					
					if (Char == '"')
					{
						if (BackslashCount % 2 == 0)
						{
							bInQuotes = !bInQuotes;
						}
					}
					BackslashCount = 0;
				}
				
				if (bInQuotes)
				{
					OutError = FText::Format(LOCTEXT("PostCreatePythonUnbalancedQuotes", "Arguments passed in (\"{0}\") contain unbalanced quotes."), FText::FromString(In));
					return false;
				}
				
				return true;
			};
		
		if (!ValidateArgsBlob(TemplateArgs, OutFailReason) || !ValidateArgsBlob(AdditionalArgs, OutFailReason))
		{
			return false;
		}
		
		TArray<FString> Argv;
		Argv.Add(AbsScriptPath);
		
		Argv.Add(TEXT("--PluginName"));
		Argv.Add(PluginName);
		
		Argv.Add(TEXT("--PluginRoot"));
		Argv.Add(PluginRoot);
		
		Argv.Add(TEXT("--IsEnginePlugin"));
		Argv.Add(IsEnginePlugin ? TEXT("1") : TEXT("0"));
		
		for (const FString& Folder : TemplateFolders)
		{
			Argv.Add(TEXT("--TemplatePath"));
			Argv.Add(FPaths::ConvertRelativePathToFull(Folder));
		}
		
		auto TokenizeArgsString = [](const FString& In, TArray<FString>& OutArgv) -> void
			{
				if (In.IsEmpty())
				{
					return; // nothing to do
				}
				
				const TCHAR* Stream = *In;
				FString Arg;
				while(FParse::Token(Stream, Arg, /*bUseEscape=*/ true))
				{
					OutArgv.Add(Arg);
				}
			};
			
		TokenizeArgsString(TemplateArgs, Argv);
		TokenizeArgsString(AdditionalArgs, Argv);
		
		auto EscapeAndQuote = [](const FString& In) -> FString
			{
				const bool bAlreadyQuoted = In.Len() >= 2 && In.StartsWith(TEXT("\"")) && In.EndsWith(TEXT("\""));
				FString InternalString = bAlreadyQuoted ? In.Mid(1, In.Len() - 2) : In;
				
				// Escape content
				InternalString = InternalString.Replace(TEXT("\\"), TEXT("\\\\"));
				InternalString = InternalString.Replace(TEXT("\""), TEXT("\\\""));
				
				const bool bNeedsQuotes = InternalString.Contains(TEXT("\"")) || InternalString.Contains(TEXT(" ")) || InternalString.Contains(TEXT("\t"));				
				return bNeedsQuotes ? FString::Printf(TEXT("\"%s\""), *InternalString) : InternalString;
			};
		
		FString CommandLine;
		for (const FString& Arg : Argv)
		{
			CommandLine += EscapeAndQuote(Arg) + TEXT(" ");
		}
		
		FPythonCommandEx PythonCommand;
		PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteFile;
		PythonCommand.Flags |= EPythonCommandFlags::Unattended;
		PythonCommand.Command = CommandLine;
		
		FScopedSlowTask Progress(/*InAmountOfWork=*/ 1.f, LOCTEXT("RunningPostCreatePython", "Running post-create Python script..."), /*bInEnabled=*/ true, *GWarn);
		Progress.MakeDialog(/*bShowCancelButton=*/ false);
		Progress.EnterProgressFrame(1.f);
		
		UE_LOGF(LogPluginUtils, Display, "Running python post-create script...");
		const bool bSuccess = PythonScriptPlugin->ExecPythonCommandEx(PythonCommand);
		UE_LOGF(LogPluginUtils, Display, "Python post-create script process completed.");
		
		for (const FPythonLogOutputEntry& LogOutputEntry : PythonCommand.LogOutput)
		{
			switch (LogOutputEntry.Type)
			{
			case EPythonLogOutputType::Error:
				UE_LOGF(LogPluginUtils, Error, "[PythonPostCreate] %ls", *LogOutputEntry.Output);
				break;
			case EPythonLogOutputType::Warning:
				UE_LOGF(LogPluginUtils, Warning, "[PythonPostCreate] %ls", *LogOutputEntry.Output);
				break;
			case EPythonLogOutputType::Info:
				UE_LOGF(LogPluginUtils, Display, "[PythonPostCreate] %ls", *LogOutputEntry.Output);
				break;
			}
		}
		
		if (!bSuccess)
		{
			OutFailReason = LOCTEXT("PostCreatePythonFailed", "Post Create Python Script failed. Check log for details.");
			return false;
		}
		
		return true;
#else
		OutFailReason = LOCTEXT("PostCreatePythonNotAvailableOnTarget", "Engine does not have Python support enabled. Python is required to run Post Create python scripts.");
		return false;
#endif
	}

	TSharedPtr<IPlugin> CreateAndLoadNewPluginInternal(const FString& PluginName, const FString& NameToReplace, const FString& PluginLocation, const FString& PluginFilePath, const FPluginUtils::FNewPluginParamsWithDescriptor& CreationParams, FPluginUtils::FLoadPluginParams& LoadParams)
	{
		if (!ensure(!PluginFilePath.IsEmpty()))
		{
			return nullptr;
		}

		if ((CreationParams.Descriptor.Modules.Num() > 0) && (CreationParams.TemplateFolders.Num() == 0))
		{
			if (LoadParams.OutFailReason)
			{
				*LoadParams.OutFailReason = LOCTEXT("CreateNewPluginParam_NoTemplateFolder", "A template folder must be specified to create a plugin with code");
			}
			return nullptr;
		}

		const FString PluginFolder = FPluginUtils::GetPluginFolder(PluginFilePath, /*bFullPath*/ true);

		// This is to fix up any internal plugin references otherwise the generated plugin content could have
		// package references back to the source template content.
		TArray<FCoreRedirect> CoreRedirectList;

		TSharedPtr<IPlugin> NewPlugin;
		bool bSucceeded = true;
		do
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			TArray<FString> NewFilePaths;

			if (!PlatformFile.DirectoryExists(*PluginFolder) && !PlatformFile.CreateDirectoryTree(*PluginFolder))
			{
				if (LoadParams.OutFailReason)
				{
					*LoadParams.OutFailReason = FText::Format(LOCTEXT("FailedToCreatePluginFolder", "Failed to create plugin folder\n{0}"), FText::FromString(PluginFolder));
				}
				bSucceeded = false;
				break;
			}

			if (CreationParams.Descriptor.bCanContainContent)
			{
				const FString PluginContentFolder = FPluginUtils::GetPluginContentFolder(PluginFilePath, /*bFullPath*/ true);
				if (!PlatformFile.DirectoryExists(*PluginContentFolder) && !PlatformFile.CreateDirectory(*PluginContentFolder))
				{
					if (LoadParams.OutFailReason)
					{
						*LoadParams.OutFailReason = FText::Format(LOCTEXT("FailedToCreatePluginContentFolder", "Failed to create plugin Content folder\n{0}"), FText::FromString(PluginContentFolder));
					}
					bSucceeded = false;
					break;
				}
			}

			// Write the uplugin file
			if (!CreationParams.Descriptor.Save(PluginFilePath, LoadParams.OutFailReason))
			{
				bSucceeded = false;
				break;
			}
			NewFilePaths.Add(PluginFilePath);

			// Copy plugin icon
			if (!CreationParams.PluginIconPath.IsEmpty())
			{
				const FString ResourcesFolder = FPluginUtils::GetPluginResourcesFolder(PluginFilePath, /*bFullPath*/ true);
				const FString DestinationPluginIconPath = FPaths::Combine(ResourcesFolder, TEXT("Icon128.png"));
				if (IFileManager::Get().Copy(*DestinationPluginIconPath, *CreationParams.PluginIconPath, /*bReplaceExisting=*/ false) != COPY_OK)
				{
					if (LoadParams.OutFailReason)
					{
						*LoadParams.OutFailReason = FText::Format(LOCTEXT("FailedToCopyPluginIcon", "Failed to copy plugin icon\nFrom: {0}\nTo: {1}"), FText::FromString(FPaths::ConvertRelativePathToFull(CreationParams.PluginIconPath)), FText::FromString(DestinationPluginIconPath));
					}
					bSucceeded = false;
					break;
				}
				NewFilePaths.Add(DestinationPluginIconPath);
			}

			// Copy template files
			if (CreationParams.TemplateFolders.Num() > 0)
			{
				GWarn->BeginSlowTask(LOCTEXT("CopyingPluginTemplate", "Copying plugin template files..."), /*ShowProgressDialog*/ true, /*bShowCancelButton*/ false);
				for (const FString& TemplateFolder : CreationParams.TemplateFolders)
				{
					if (!PluginUtils::CopyPluginTemplateFolder(*PluginFolder, *TemplateFolder, PluginName, NameToReplace, CoreRedirectList, NewFilePaths, LoadParams.OutFailReason))
					{
						if (LoadParams.OutFailReason)
						{
							*LoadParams.OutFailReason = FText::Format(LOCTEXT("FailedToCopyPluginTemplate", "Failed to copy plugin template files\nFrom: {0}\nTo: {1}\n{2}"), FText::FromString(FPaths::ConvertRelativePathToFull(TemplateFolder)), FText::FromString(PluginFolder), *LoadParams.OutFailReason);
						}
						bSucceeded = false;
						break;
					}
				}
				GWarn->EndSlowTask();
			}

			if (!bSucceeded)
			{
				break;
			}

			if (!CoreRedirectList.IsEmpty())
			{
				// The package redirectors need to be in place before loading any content in the generated plugin.
				FCoreRedirects::AddRedirectList(CoreRedirectList, TEXT("GenerateNewPlugin"));
			}

			// Compile plugin code
			if (CreationParams.Descriptor.Modules.Num() > 0)
			{
				const FString ProjectFileName = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::GetProjectFilePath());

				// Internal plugins will be found and built automatically, and using the -Plugin param will mark them as external plugins causing them to fail to load.
				const bool bIsEnginePlugin = FPaths::IsUnderDirectory(PluginFilePath, FPaths::EnginePluginsDir());
				const bool bIsProjectPlugin = FPaths::IsUnderDirectory(PluginFilePath, FPaths::ProjectPluginsDir());
				const FString LocalPluginArgument = (bIsEnginePlugin || bIsProjectPlugin) ? FString::Printf(TEXT("-BuildPluginAsLocal ")) : FString();

				const FString Arguments = FString::Printf(TEXT("%s %s %s -Plugin=\"%s\" -Project=\"%s\" %s-Progress -NoHotReloadFromIDE"), FPlatformMisc::GetUBTTargetName(), FModuleManager::Get().GetUBTConfiguration(), FPlatformMisc::GetUBTPlatform(), *PluginFilePath, *ProjectFileName, *LocalPluginArgument);

				if (!FDesktopPlatformModule::Get()->RunUnrealBuildTool(FText::Format(LOCTEXT("CompilingPlugin", "Compiling {0} plugin..."), FText::FromString(PluginName)), FPaths::RootDir(), Arguments, GWarn))
				{
					if (LoadParams.OutFailReason)
					{
						*LoadParams.OutFailReason = LOCTEXT("FailedToCompilePlugin", "Failed to compile plugin source code. See output log for more information.");
					}
					bSucceeded = false;
					break;
				}

				// Reset the module paths cache. For unique build environments, the modules may be generated to the project binaries directory.
				FModuleManager::Get().ResetModulePathsCache();

				// Generate project files if we happen to be using a project file.
				if (!FDesktopPlatformModule::Get()->GenerateProjectFiles(FPaths::RootDir(), FPaths::GetProjectFilePath(), GWarn))
				{
					if (LoadParams.OutFailReason)
					{
						*LoadParams.OutFailReason = LOCTEXT("FailedToGenerateProjectFiles", "Failed to generate project files");
					}
					bSucceeded = false;
					break;
				}
			}

			// Load the new plugin
			NewPlugin = PluginUtils::LoadPluginInternal(PluginName, PluginLocation, PluginFilePath, LoadParams, /*bIsNewPlugin*/ true);
			if (!NewPlugin)
			{
				bSucceeded = false;
				break;
			}

			// Fix any content that was added to the plugin
			if (CreationParams.Descriptor.bCanContainContent)
			{	
				GWarn->BeginSlowTask(LOCTEXT("LoadingContent", "Loading Content..."), /*ShowProgressDialog*/ true, /*bShowCancelButton*/ false);
				TMap<FString, FString> ModifiedAssetPaths;
				PluginUtils::FixupPluginTemplateAssets(PluginName, NameToReplace, ModifiedAssetPaths);

				for (FString& CopiedFilePath : NewFilePaths)
				{
					FString FullCopiedPath = FPaths::ConvertRelativePathToFull(CopiedFilePath);
					FPaths::RemoveDuplicateSlashes(FullCopiedPath);
					if (FString* NewPath = ModifiedAssetPaths.Find(FullCopiedPath))
					{
						CopiedFilePath = *NewPath;
					}
				}
				GWarn->EndSlowTask();
			}

			// Add the plugin files to source control if the project is configured for it
			if (USourceControlHelpers::IsAvailable())
			{
				GWarn->BeginSlowTask(LOCTEXT("AddingFilesToSourceControl", "Adding to Revision Control..."), /*ShowProgressDialog*/ true, /*bShowCancelButton*/ false);
				USourceControlHelpers::MarkFilesForAdd(NewFilePaths);
				GWarn->EndSlowTask();
			}
			
		} while (false);

		if (!CoreRedirectList.IsEmpty())
		{
			// Remove the package redirectors now that all the generated content has been saved.
			FCoreRedirects::RemoveRedirectList(CoreRedirectList, TEXT("GenerateNewPlugin"));
		}
			
		if (!CreationParams.PostCreatePythonScriptPath.IsEmpty())
		{
			const bool bIsEnginePlugin = FPaths::IsUnderDirectory(PluginFilePath, FPaths::EnginePluginsDir());
				
			FText OutPythonScriptFailReason;
			const bool bPythonOk = RunPostCreatePythonScript(
				PluginName,
				PluginFolder,
				CreationParams.TemplateFolders,
				CreationParams.PostCreatePythonScriptPath,
				CreationParams.PostCreatePythonScriptArguments,
				CreationParams.PostCreatePythonScriptAdditionalArguments,
				bIsEnginePlugin,
				OutPythonScriptFailReason
			);

			if (!bPythonOk)
			{
				if (LoadParams.OutFailReason)
				{
					*LoadParams.OutFailReason = OutPythonScriptFailReason;
				}
					
				bSucceeded = false;
			}
		}

		if (!bSucceeded)
		{
			// Delete the plugin folder is something goes wrong during the plugin creation.
			IFileManager::Get().DeleteDirectory(*PluginFolder, /*RequireExists*/ false, /*Tree*/ true);
			if (NewPlugin)
			{
				// Refresh plugins if the new plugin was registered, but we decide to delete its files.
				IPluginManager::Get().RefreshPluginsList();
				NewPlugin.Reset();
				ensure(!IPluginManager::Get().FindPlugin(PluginName));
			}
		}

		return NewPlugin;
	}

	bool ValidateNewPluginFilePath(const FString& PluginFilePath, FText* FailReason)
	{
		// Check if a .uplugin file exists
		if (!PluginFilePath.IsEmpty() && FPaths::FileExists(PluginFilePath))
		{
			if (FailReason)
			{
				*FailReason = FText::Format(LOCTEXT("PluginPathExists", "Plugin already exists at this location\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(PluginFilePath)));
			}
			return false;
		}

		return true;
	}

	bool ValidateNewPluginFolder(const FString& PluginFolder, FText* FailReason)
	{
		// Check there isn't an existing file along the plugin folder path that would prevent creating the directory tree
		{
			FString ExistingFilePath = PluginFolder;
			while (!ExistingFilePath.IsEmpty())
			{
				if (FPaths::FileExists(ExistingFilePath))
				{
					break;
				}
				ExistingFilePath = FPaths::GetPath(ExistingFilePath);
			}

			if (!ExistingFilePath.IsEmpty())
			{
				if (FailReason)
				{
					*FailReason = FText::Format(LOCTEXT("PluginLocationIsFile", "Plugin location is invalid because the following file is in the way\n{0}"), FText::FromString(ExistingFilePath));
				}
				return false;
			}
		}

		return true;
	}

	bool ValidateNewPluginName(const FString& PluginName, FText* FailReason)
	{
		// Check to see if a discovered plugin with this name exists (at any path)
		if (TSharedPtr<IPlugin> ExistingPlugin = IPluginManager::Get().FindPlugin(PluginName))
		{
			if (FailReason)
			{
				*FailReason = FText::Format(LOCTEXT("PluginNameAlreadyInUse", "Plugin name is already in use by\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(ExistingPlugin->GetDescriptorFileName())));
			}
			return false;
		}

		// Check if a mount point with the same name exists
		{
			FString MountPoint;
			MountPoint.Reserve(PluginName.Len() + 2);
			MountPoint.AppendChar(TEXT('/'));
			MountPoint.Append(PluginName);
			MountPoint.AppendChar(TEXT('/'));
			if (FPackageName::MountPointExists(MountPoint))
			{
				if (FailReason)
				{
					*FailReason = FText::Format(LOCTEXT("MountPointExists", "Mount point {0} already exists"), FText::FromString(MountPoint));
				}
				return false;
			}
		}

		return true;
	}
}

FString FPluginUtils::GetPluginFolder(const FString& PluginLocation, const FString& PluginName, bool bFullPath)
{
	FString PluginFolder = FPaths::Combine(PluginLocation, PluginName);
	if (bFullPath)
	{
		PluginFolder = FPaths::ConvertRelativePathToFull(PluginFolder);
	}
	FPaths::MakePlatformFilename(PluginFolder);
	return PluginFolder;
}

FString FPluginUtils::GetPluginFolder(const FString& PluginFilePath, bool bFullPath)
{
	FString PluginFolder = FPaths::GetPath(PluginFilePath);
	if (bFullPath)
	{
		PluginFolder = FPaths::ConvertRelativePathToFull(PluginFolder);
	}
	FPaths::MakePlatformFilename(PluginFolder);
	return PluginFolder;
}

FString FPluginUtils::GetPluginFilePath(const FString& PluginLocation, const FString& PluginName, bool bFullPath)
{
	FString PluginFilePath = FPaths::Combine(PluginLocation, PluginName, (PluginName + FPluginDescriptor::GetFileExtension()));
	if (bFullPath)
	{
		PluginFilePath = FPaths::ConvertRelativePathToFull(PluginFilePath);
	}
	FPaths::MakePlatformFilename(PluginFilePath);
	return PluginFilePath;
}

FString FPluginUtils::GetPluginName(const FString& PluginFilePath)
{
	return FPaths::GetBaseFilename(PluginFilePath);
}

FString FPluginUtils::GetPluginContentFolder(const FString& PluginLocation, const FString& PluginName, bool bFullPath)
{
	FString PluginContentFolder = FPaths::Combine(PluginLocation, PluginName, TEXT("Content"));
	if (bFullPath)
	{
		PluginContentFolder = FPaths::ConvertRelativePathToFull(PluginContentFolder);
	}
	FPaths::MakePlatformFilename(PluginContentFolder);
	return PluginContentFolder;
}

FString FPluginUtils::GetPluginContentFolder(const FString& PluginFilePath, bool bFullPath)
{
	FString PluginContentFolder = FPaths::Combine(FPaths::GetPath(PluginFilePath), TEXT("Content"));
	if (bFullPath)
	{
		PluginContentFolder = FPaths::ConvertRelativePathToFull(PluginContentFolder);
	}
	FPaths::MakePlatformFilename(PluginContentFolder);
	return PluginContentFolder;
}

FString FPluginUtils::GetPluginResourcesFolder(const FString& PluginLocation, const FString& PluginName, bool bFullPath)
{
	FString PluginResourcesFolder = FPaths::Combine(PluginLocation, PluginName, TEXT("Resources"));
	if (bFullPath)
	{
		PluginResourcesFolder = FPaths::ConvertRelativePathToFull(PluginResourcesFolder);
	}
	FPaths::MakePlatformFilename(PluginResourcesFolder);
	return PluginResourcesFolder;
}

FString FPluginUtils::GetPluginResourcesFolder(const FString& PluginFilePath, bool bFullPath)
{
	FString PluginResourcesFolder = FPaths::Combine(FPaths::GetPath(PluginFilePath), TEXT("Resources"));
	if (bFullPath)
	{
		PluginResourcesFolder = FPaths::ConvertRelativePathToFull(PluginResourcesFolder);
	}
	FPaths::MakePlatformFilename(PluginResourcesFolder);
	return PluginResourcesFolder;
}

TSharedPtr<IPlugin> FPluginUtils::CreateAndLoadNewPlugin(const FString& PluginName, const FString& PluginLocation, const FNewPluginParams& CreationParams, FLoadPluginParams& LoadParams)
{
	FPluginUtils::FNewPluginParamsWithDescriptor ExCreationParams;
	PluginUtils::InitializeNewPluginParamsWithDescriptor(PluginName, CreationParams, ExCreationParams);

	return CreateAndLoadNewPlugin(PluginName, PluginLocation, ExCreationParams, LoadParams);
}

TSharedPtr<IPlugin> FPluginUtils::CreateAndLoadNewPlugin(const FString& PluginName, const FString& PluginLocation, const FNewPluginParamsWithDescriptor& CreationParams, FLoadPluginParams& LoadParams)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CreateAndLoadNewPlugin(PluginName, !CreationParams.NameToReplace.IsEmpty() ? CreationParams.NameToReplace : PluginName, PluginLocation, CreationParams, LoadParams);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSharedPtr<IPlugin> FPluginUtils::CreateAndLoadNewPlugin(const FString& PluginName, const FString& NameToReplace, const FString& PluginLocation, const FNewPluginParamsWithDescriptor& CreationParams, FLoadPluginParams& LoadParams)
{
	// Early validations on new plugin params
	if (PluginName.IsEmpty())
	{
		if (LoadParams.OutFailReason)
		{
			*LoadParams.OutFailReason = LOCTEXT("CreateNewPluginParam_NoPluginName", "Missing plugin name");
		}
		return nullptr;
	}

	if (PluginLocation.IsEmpty())
	{
		if (LoadParams.OutFailReason)
		{
			*LoadParams.OutFailReason = LOCTEXT("CreateNewPluginParam_NoPluginLocation", "Missing plugin location");
		}
		return nullptr;
	}

	if (!FPluginUtils::ValidateNewPluginNameAndLocation(PluginName, PluginLocation, LoadParams.OutFailReason))
	{
		return nullptr;
	}

	const FString PluginFilePath = FPluginUtils::GetPluginFilePath(PluginLocation, PluginName, /*bFullPath*/ true);

	return PluginUtils::CreateAndLoadNewPluginInternal(PluginName, NameToReplace, PluginLocation, PluginFilePath, CreationParams, LoadParams);
}

TSharedPtr<IPlugin> FPluginUtils::CreateAndLoadNewPlugin(const FString& PluginFilePath, const FNewPluginParams& CreationParams, FLoadPluginParams& LoadParams)
{
	FPluginUtils::FNewPluginParamsWithDescriptor ExCreationParams;
	PluginUtils::InitializeNewPluginParamsWithDescriptor(GetPluginName(PluginFilePath), CreationParams, ExCreationParams);

	return CreateAndLoadNewPlugin(PluginFilePath, ExCreationParams, LoadParams);
}

TSharedPtr<IPlugin> FPluginUtils::CreateAndLoadNewPlugin(const FString& PluginFilePath, const FNewPluginParamsWithDescriptor& CreationParams, FLoadPluginParams& LoadParams)
{
	// Early validations on new plugin params
	if (PluginFilePath.IsEmpty())
	{
		if (LoadParams.OutFailReason)
		{
			*LoadParams.OutFailReason = LOCTEXT("CreateNewPluginParam_NoPluginFilePath", "Missing plugin file name");
		}
		return nullptr;
	}

	if (!FPluginUtils::ValidateNewPluginFilePath(PluginFilePath, LoadParams.OutFailReason))
	{
		return nullptr;
	}

	const FString PluginName = GetPluginName(PluginFilePath);

	FString FullPluginFilePath = FPaths::ConvertRelativePathToFull(PluginFilePath);
	FPaths::MakePlatformFilename(FullPluginFilePath);

	return PluginUtils::CreateAndLoadNewPluginInternal(PluginName, !CreationParams.NameToReplace.IsEmpty() ? CreationParams.NameToReplace : PluginName, FString(), FullPluginFilePath, CreationParams, LoadParams);
}

TSharedPtr<IPlugin> FPluginUtils::LoadPlugin(const FString& PluginName, const FString& PluginLocation, FLoadPluginParams& LoadParams)
{
	return LoadPlugin(FPluginUtils::GetPluginFilePath(PluginLocation, PluginName), LoadParams);
}

TSharedPtr<IPlugin> FPluginUtils::LoadPlugin(const FString& PluginFilePath, FLoadPluginParams& LoadParams)
{
	// Valide that the uplugin file exists.
	if (!FPaths::FileExists(PluginFilePath))
	{
		if (LoadParams.OutFailReason)
		{
			*LoadParams.OutFailReason = FText::Format(LOCTEXT("PluginFileDoesNotExist", "Plugin file does not exist\n{0}"), FText::FromString(FPaths::ConvertRelativePathToFull(PluginFilePath)));
		}
		return nullptr;
	}

	const FString PluginName = GetPluginName(PluginFilePath);
	if (!IsValidPluginName(PluginName, LoadParams.OutFailReason))
	{
		return nullptr;
	}

	const FString PluginLocation = FPaths::GetPath(FPaths::GetPath(PluginFilePath));
	return PluginUtils::LoadPluginInternal(PluginName, PluginLocation, PluginFilePath, LoadParams, /*bIsNewPlugin*/ false);
}

TSharedPtr<IPlugin> FPluginUtils::FindLoadedPlugin(const FString& PluginDescriptorFileName)
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(FPaths::GetBaseFilename(PluginDescriptorFileName));
	if (Plugin &&
		Plugin->IsEnabled() &&
		FPaths::IsSamePath(Plugin->GetDescriptorFileName(), PluginDescriptorFileName) &&
		(!Plugin->CanContainContent() || FPackageName::MountPointExists(Plugin->GetMountedAssetPath())))
	{
		return Plugin;
	}
	return TSharedPtr<IPlugin>();
}

TSharedPtr<IPlugin> FPluginUtils::FindPluginFromPackagePath(FName PackagePath)
{
	FNameBuilder PackagePathNameBuilder(PackagePath);
	return FindPluginFromPackagePath(PackagePathNameBuilder);
}

TSharedPtr<IPlugin> FPluginUtils::FindPluginFromPackagePath(FStringView PackagePath)
{
	TSharedPtr<IPlugin> Plugin;
	static const FStringView ScriptPath = TEXTVIEW("/Script/");
	if (PackagePath.StartsWith(ScriptPath))
	{
		const FName ModuleName(PackagePath.RightChop(ScriptPath.Len()));
		Plugin = IPluginManager::Get().GetModuleOwnerPlugin(ModuleName);
	}
	else
	{
		const FStringView MountPoint = FPathViews::GetMountPointNameFromPath(PackagePath);
		Plugin = IPluginManager::Get().FindPlugin(MountPoint);
	}
	return Plugin;
}

bool FPluginUtils::UnloadPlugin(const TSharedRef<IPlugin>& Plugin, FText* OutFailReason /*= nullptr*/)
{
	return UnloadPlugins({ Plugin }, OutFailReason);
}

bool FPluginUtils::UnloadPlugin(const FString& PluginName, FText* OutFailReason /*= nullptr*/)
{
	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
	{
		return UnloadPlugin(Plugin.ToSharedRef(), OutFailReason);
	}
	return true;
}

bool FPluginUtils::UnloadPlugins(const TConstArrayView<TSharedRef<IPlugin>> Plugins, FText* OutFailReason /*= nullptr*/)
{
	if (Plugins.Num() == 0)
	{
		return true;
	}

	{
		FText ErrorMsg;
		if (!UnloadPluginsAssets(Plugins, &ErrorMsg))
		{
			// If some assets fail to unload, log an error, but unmount the plugins anyway
			UE_LOGF(LogPluginUtils, Error, "Failed to unload some assets prior to unmounting plugins\n%ls", *ErrorMsg.ToString());
		}
	}

	// Unmount the plugins
	//
	bool bSuccess = true;
	{
		FTextBuilder ErrorBuilder;
		bool bPluginUnmounted = false;

		for (const TSharedRef<IPlugin>& Plugin : Plugins)
		{
			if (Plugin->IsMounted())
			{
				bPluginUnmounted = true;

				FText FailReason;
				if (!IPluginManager::Get().UnmountExplicitlyLoadedPlugin(Plugin->GetName(), &FailReason))
				{
					UE_LOGF(LogPluginUtils, Error, "Plugin %ls cannot be unloaded: %ls", *Plugin->GetName(), *FailReason.ToString());
					ErrorBuilder.AppendLine(FailReason);
					bSuccess = false;
				}
			}
		}

		if (bPluginUnmounted)
		{
			IPluginManager::Get().RefreshPluginsList();
		}

		if (!bSuccess && OutFailReason)
		{
			*OutFailReason = ErrorBuilder.ToText();
		}
	}
	return bSuccess;
}

bool FPluginUtils::UnloadPlugins(const TConstArrayView<FString> PluginNames, FText* OutFailReason /*= nullptr*/)
{
	TArray<TSharedRef<IPlugin>> Plugins;
	Plugins.Reserve(PluginNames.Num());
	for (const FString& PluginName : PluginNames)
	{
		if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
		{
			Plugins.Add(Plugin.ToSharedRef());
		}
	}
	return UnloadPlugins(Plugins, OutFailReason);
}

bool FPluginUtils::UnloadPluginAssets(const TSharedRef<IPlugin>& Plugin, FText* OutFailReason /*= nullptr*/)
{
	return UnloadPluginAssets(Plugin->GetName(), OutFailReason);
}

bool FPluginUtils::UnloadPluginAssets(const FString& PluginName, FText* OutFailReason /*= nullptr*/)
{
	TSet<FString> PluginNames;
	PluginNames.Reserve(1);
	PluginNames.Add(PluginName);
	return UnloadPluginsAssets(PluginNames, OutFailReason);
}

bool FPluginUtils::UnloadPluginsAssets(const TConstArrayView<TSharedRef<IPlugin>> Plugins, FText* OutFailReason /*= nullptr*/)
{
	TSet<FString> PluginNames;
	PluginNames.Reserve(Plugins.Num());
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		if (Plugin->IsMounted())
		{
			PluginNames.Add(Plugin->GetName());
		}
	}

	return UnloadPluginsAssets(PluginNames, OutFailReason);
}

bool FPluginUtils::UnloadPluginsAssets(const TSet<FString>& PluginNames, FText* OutFailReason /*= nullptr*/)
{
	bool bSuccess = true;
	if (!PluginNames.IsEmpty())
	{
		const double StartTime = FPlatformTime::Seconds();

		TArray<UPackage*> PackagesToUnload;
		for (TObjectIterator<UPackage> It; It; ++It)
		{
			const FNameBuilder PackageName(It->GetFName());
			const FStringView PackageMountPointName = FPathViews::GetMountPointNameFromPath(PackageName);
			if (PluginNames.ContainsByHash(GetTypeHash(PackageMountPointName), PackageMountPointName))
			{
				PackagesToUnload.Add(*It);
			}
		}

		if (PackagesToUnload.Num() > 0)
		{
			FText ErrorMsg;
			UPackageTools::UnloadPackages(PackagesToUnload, ErrorMsg, /*bUnloadDirtyPackages=*/true);

			// @note UnloadPackages returned bool indicates whether some packages were unloaded
			// To tell whether all packages were successfully unloaded we must check the ErrorMsg output param
			if (!ErrorMsg.IsEmpty())
			{
				if (OutFailReason)
				{
					*OutFailReason = MoveTemp(ErrorMsg);
				}
				bSuccess = false;
			}
		}

		UE_LOGF(LogPluginUtils, Log, "Unloading assets from %d plugins took %0.2f sec", PluginNames.Num(), FPlatformTime::Seconds() - StartTime);
	}
	return bSuccess;
}

bool FPluginUtils::UnloadPluginsAssets(const TConstArrayView<FString> PluginNames, FText* OutFailReason /*= nullptr*/)
{
	TSet<FString> PluginNamesSet;
	PluginNamesSet.Reserve(PluginNames.Num());
	for (const FString& PluginName : PluginNames)
	{
		PluginNamesSet.Add(PluginName);
	}
	return UnloadPluginsAssets(PluginNamesSet, OutFailReason);
}

bool FPluginUtils::AddToPluginSearchPathIfNeeded(const FString& Dir, bool bRefreshPlugins, bool bUpdateProjectFile)
{
	bool bSearchPathChanged = false;

	const bool bIsEnginePlugin = FPaths::IsUnderDirectory(Dir, FPaths::EnginePluginsDir());
	const bool bIsProjectPlugin = FPaths::IsUnderDirectory(Dir, FPaths::ProjectPluginsDir());
	const bool bIsModPlugin = FPaths::IsUnderDirectory(Dir, FPaths::ProjectModsDir());
	const bool bIsEnterprisePlugin = FPaths::IsUnderDirectory(Dir, FPaths::EnterprisePluginsDir());
	
	if (bIsEnginePlugin || bIsProjectPlugin || bIsModPlugin || bIsEnterprisePlugin)
	{
		return false;
	}
	
	if (bUpdateProjectFile)
	{
		bool bNeedToUpdate = true;
		for (const FString& AdditionalDir : IProjectManager::Get().GetAdditionalPluginDirectories())
		{
			if (FPaths::IsUnderDirectory(Dir, AdditionalDir))
			{
				bNeedToUpdate = false;
				break;
			}
		}

		if (bNeedToUpdate)
		{
			bSearchPathChanged = GameProjectUtils::UpdateAdditionalPluginDirectory(Dir, /*bAdd*/ true);
		}
	}
	else
	{
		bool bNeedToUpdate = true;
		for (const FString& AdditionalDir : IPluginManager::Get().GetAdditionalPluginSearchPaths())
		{
			if (FPaths::IsUnderDirectory(Dir, AdditionalDir))
			{
				bNeedToUpdate = false;
				break;
			}
		}			

		if (bNeedToUpdate)
		{
			bSearchPathChanged = IPluginManager::Get().AddPluginSearchPath(Dir, /*bShouldRefresh*/ false);
		}
	}

	if (bSearchPathChanged && bRefreshPlugins)
	{
		IPluginManager::Get().RefreshPluginsList();
	}

	return bSearchPathChanged;
}

bool FPluginUtils::ValidateNewPluginNameAndLocation(const FString& PluginName, const FString& PluginLocation /*= FString()*/, FText* FailReason /*= nullptr*/)
{
	// Check whether the plugin name is valid
	if (!IsValidPluginName(PluginName, FailReason))
	{
		return false;
	}

	if (!PluginLocation.IsEmpty())
	{
		if (!PluginUtils::ValidateNewPluginFilePath(FPluginUtils::GetPluginFilePath(PluginLocation, PluginName), FailReason))
		{
			return false;
		}

		// Check that the plugin location is a valid path (it doesn't have to exist; it will be created if needed)
		if (!FPaths::ValidatePath(PluginLocation, FailReason))
		{
			if (FailReason)
			{
				*FailReason = FText::Format(LOCTEXT("PluginLocationIsNotValidPath", "Plugin location is not a valid path\n{0}"), *FailReason);
			}
			return false;
		}

		if (!PluginUtils::ValidateNewPluginFolder(FPluginUtils::GetPluginFolder(PluginLocation, PluginName, true /*bFullPath*/), FailReason))
		{
			return false;
		}
	}

	if (!PluginUtils::ValidateNewPluginName(PluginName, FailReason))
	{
		return false;
	}

	return true;
}

bool FPluginUtils::ValidateNewPluginFilePath(const FString& PluginFilePath, FText* FailReason /*= nullptr*/)
{
	const FString PluginName = GetPluginName(PluginFilePath);

	// Check whether the plugin name is valid
	if (!IsValidPluginName(PluginName, FailReason))
	{
		return false;
	}

	if (!PluginUtils::ValidateNewPluginFilePath(PluginFilePath, FailReason))
	{
		return false;
	}

	if (!PluginUtils::ValidateNewPluginFolder(FPluginUtils::GetPluginFolder(PluginFilePath, true /*bFullPath*/), FailReason))
	{
		return false;
	}

	if (!PluginUtils::ValidateNewPluginName(PluginName, FailReason))
	{
		return false;
	}

	return true;
}

bool FPluginUtils::IsValidPluginName(const FString& PluginName, FText* FailReason /*=nullptr*/, const FText* PluginTermReplacement /*=nullptr*/)
{
	static const FText PluginTerm = LOCTEXT("PluginTerm", "Plugin");
	const FText& PluginTermToUse = PluginTermReplacement ? *PluginTermReplacement : PluginTerm;

	// Cannot be empty
	if (PluginName.IsEmpty())
	{
		if (FailReason)
		{
			*FailReason = FText::Format(LOCTEXT("PluginNameIsEmpty", "{0} name cannot be empty"), PluginTermToUse);
		}
		return false;
	}

	// Only allow alphanumeric characters and underscore in the name
	FString IllegalCharacters;
	for (int32 CharIdx = 0; CharIdx < PluginName.Len(); ++CharIdx)
	{
		const FString& Char = PluginName.Mid(CharIdx, 1);
		if (!FChar::IsAlnum(Char[0]) && Char != TEXT("_") && Char != TEXT("-"))
		{
			if (!IllegalCharacters.Contains(Char))
			{
				IllegalCharacters += Char;
			}
		}
	}

	if (IllegalCharacters.Len() > 0)
	{
		if (FailReason)
		{
			*FailReason = FText::Format(LOCTEXT("PluginNameContainsIllegalCharacters", "{0} name cannot contain characters such as \"{1}\""), PluginTermToUse, FText::FromString(IllegalCharacters));
		}
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif //if WITH_EDITOR
