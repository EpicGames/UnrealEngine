// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PluginDescriptor.h"
#include "ModuleDescriptor.h"

class IPlugin;

class FPluginUtils
{
public:
	/**
	 * Returns the plugin folder.
	 * @param PluginLocation directory that contains the plugin folder
	 * @param PluginName name of the plugin
	 * @param bFullPath ensures a full path is returned
	 */
	static PLUGINUTILS_API FString GetPluginFolder(const FString& PluginLocation, const FString& PluginName, bool bFullPath = false);

	/**
	 * Returns the plugin folder from its uplugin file path.
	 * @param PluginFilePath Plugin descriptor file path
	 */
	static PLUGINUTILS_API FString GetPluginFolder(const FString& PluginFilePath, bool bFullPath = false);

	/** 
	 * Returns the uplugin file path.
	 * @param PluginLocation directory that contains the plugin folder
	 * @param PluginName name of the plugin
	 * @param bFullPath ensures a full path is returned
	 */
	static PLUGINUTILS_API FString GetPluginFilePath(const FString& PluginLocation, const FString& PluginName, bool bFullPath = false);

	/**
	 * Returns the plugin name from its uplugin file path.
	 * @param PluginFilePath Plugin descriptor file path
	 */
	static PLUGINUTILS_API FString GetPluginName(const FString& PluginFilePath);

	/**
	 * Returns the plugin Content folder.
	 * @param PluginLocation directory that contains the plugin folder
	 * @param PluginName name of the plugin
	 * @param bFullPath ensures a full path is returned
	 */
	static PLUGINUTILS_API FString GetPluginContentFolder(const FString& PluginLocation, const FString& PluginName, bool bFullPath = false);

	/**
	 * Returns the plugin Content folder from its uplugin file path.
	 * @param PluginFilePath Plugin descriptor file path
	 */
	static PLUGINUTILS_API FString GetPluginContentFolder(const FString& PluginFilePath, bool bFullPath = false);

	/**
	 * Returns the plugin Resources folder.
	 * @param PluginLocation directory that contains the plugin folder
	 * @param PluginName name of the plugin
	 * @param bFullPath ensures a full path is returned
	 */
	static PLUGINUTILS_API FString GetPluginResourcesFolder(const FString& PluginLocation, const FString& PluginName, bool bFullPath = false);

	/**
	 * Returns the plugin Resources folder from its uplugin file path.
	 * @param PluginFilePath Plugin descriptor file path
	 */
	static PLUGINUTILS_API FString GetPluginResourcesFolder(const FString& PluginFilePath, bool bFullPath = false);

	/**
	 * Parameters for creating a new plugin.
	 */
	struct FNewPluginParams
	{
		/** The author of this plugin */
		FString CreatedBy;

		/** Hyperlink for the author's website  */
		FString CreatedByURL;

		/** A friendly name for this plugin. Set to the plugin name by default */
		FString FriendlyName;

		/** A description for this plugin */
		FString Description;

		/** Path to plugin icon to copy in the plugin resources folder */
		FString PluginIconPath;

		/** 
		 * Folders containing template files to copy into the plugin folder (Required if bHasModules).
		 * Occurrences of the string PLUGIN_NAME in the filename or file content will be replaced by the plugin name. 
		 */
		TArray<FString> TemplateFolders;

		/** Marks this content as being in beta */
		bool bIsBetaVersion = false;

		/** Can this plugin contain content */
		bool bCanContainContent = false;

		/** Can this plugin contain Verse */
		bool bCanContainVerse = false;

		/** Does this plugin have Source files? */
		bool bHasModules = false;

		/**
		 * When true, this plugin's modules will not be loaded automatically nor will it's content be mounted automatically.
		 * It will load/mount when explicitly requested and LoadingPhases will be ignored.
		 */
		bool bExplicitelyLoaded = false;

		/** The Verse path to the root of this plugin's content directory */
		FString VersePath;

		/** The version of the Verse language that this plugin targets.
			If no value is specified, the latest stable version is used. */
		TOptional<uint32> VerseVersion;

		/** Whether this plugin has SceneGraph enabled, which impacts the generated Verse Asset Digest. */
		bool bEnableSceneGraph = false;

		/** Whether this plugin has SceneGraph Animation enabled, which impacts the generated Verse Asset Digest. */
		bool bEnableSceneGraphAnimation = false;

		/** If to generate Verse source code definitions from assets contained in this plugin */
		bool bEnableVerseAssetReflection = false;

		/** enable iad for this plugin */
		bool bEnableIAD = false;

		/** Whether this plugin should be enabled/disabled by default for any project. */
		EPluginEnabledByDefault EnabledByDefault = EPluginEnabledByDefault::Unspecified;

		/** If this plugin has Source, what is the type of Source included (so it can potentially be excluded in the right builds) */
		EHostType::Type ModuleDescriptorType = EHostType::Runtime;

		/** If this plugin has Source, when should the module be loaded (may need to be earlier than default if used in blueprints) */
		ELoadingPhase::Type LoadingPhase = ELoadingPhase::Default;
		
		/**
		 * Strings used to run a python script once the plugin has been created.
		 * If PostCreatePythonScriptPath is empty, it will not run anything.
		 */
		FString PostCreatePythonScriptPath;
		FString PostCreatePythonScriptArguments;
		FString PostCreatePythonScriptAdditionalArguments;
	};

	/**
	 * Parameters for creating a new plugin.
	 */
	struct FNewPluginParamsWithDescriptor
	{
		/** The description of the plugin */
		FPluginDescriptor Descriptor;

		/** Path to plugin icon to copy in the plugin resources folder */
		FString PluginIconPath;

		/**
		 * Folders containing template files to copy into the plugin folder (Required if Descriptor.Modules is not empty).
		 * Occurrences of the string PLUGIN_NAME in the filename or file content will be replaced by NameToReplace.
		 */
		TArray<FString> TemplateFolders;

		/**
		 * String used to replace PLUGIN_NAME in templates.
		 * If empty, the plugin name will be used.
		 */
		FString NameToReplace;
		
		/**
		 * Strings used to run a python script once the plugin has been created.
		 * If PostCreatePythonScriptPath is empty, it will not run anything.
		 */
		FString PostCreatePythonScriptPath;
		FString PostCreatePythonScriptArguments;
		FString PostCreatePythonScriptAdditionalArguments;
	};

	/**
	 * Parameters for loading/mounting a plugin
	 */
	struct FLoadPluginParams
	{
		/** Whether to synchronously scan all assets in the plugin */
		bool bSynchronousAssetsScan = false;

		/** Whether to select the plugin Content folder (if any) in the content browser */
		bool bSelectInContentBrowser = false;

		/** Whether to enable the plugin in the current project config */
		bool bEnablePluginInProject = false;

		/**
		 * Whether to update the project additional plugin directories (persistently saved in uproject file)
		 * if the plugin location is not under the engine or project plugin folder
		 */
		bool bUpdateProjectPluginSearchPath = false;

		/** Outputs whether the plugin was already loaded */
		bool bOutAlreadyLoaded =  false;

		/** Outputs the reason the plugin loading failed (if applicable) */
		FText* OutFailReason = nullptr;
	};

	/**
	 * Helper to create and load a new plugin
	 * @param PluginName Plugin name
	 * @param PluginLocation Directory that contains the plugin folder
	 * @param CreationParams Plugin creation parameters
	 * @param LoadParams Plugin loading parameters
	 * @return The newly created plugin. If something goes wrong during the creation process, the plugin folder gets deleted and null is returned.
	 * @note LoadParams.OutFailReason outputs the reason the plugin creation or loading failed (if applicable)
	 * @note Will fail if the plugin already exists
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> CreateAndLoadNewPlugin(const FString& PluginName, const FString& PluginLocation, const FNewPluginParams& CreationParams, FLoadPluginParams& LoadParams);

	/**
	 * Helper to create and load a new plugin
	 * @param PluginName Plugin name
	 * @param PluginLocation Directory that contains the plugin folder
	 * @param CreationParams Plugin creation parameters
	 * @param LoadParams Plugin loading parameters
	 * @return The newly created plugin. If something goes wrong during the creation process, the plugin folder gets deleted and null is returned.
	 * @note LoadParams.OutFailReason outputs the reason the plugin creation or loading failed (if applicable)
	 * @note Will fail if the plugin already exists
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> CreateAndLoadNewPlugin(const FString& PluginName, const FString& PluginLocation, const FNewPluginParamsWithDescriptor& CreationParams, FLoadPluginParams& LoadParams);
	
	/**
	 * Helper to create and load a new plugin
	 * @param PluginName Plugin name
	 * @param NameToReplace what will be used to replace PLUGIN_NAME in the given templates. Pass in PluginName here for default behavior
	 * @param PluginLocation Directory that contains the plugin folder
	 * @param CreationParams Plugin creation parameters
	 * @param LoadParams Plugin loading parameters
	 * @return The newly created plugin. If something goes wrong during the creation process, the plugin folder gets deleted and null is returned.
	 * @note LoadParams.OutFailReason outputs the reason the plugin creation or loading failed (if applicable)
	 * @note Will fail if the plugin already exists
	 */
	UE_DEPRECATED(5.6, "Please set NameToReplace in the FNewPluginParamsWithDescriptor and call the CreateAndLoadNewPlugin overload without the NameToReplace parameter.")
	static PLUGINUTILS_API TSharedPtr<IPlugin> CreateAndLoadNewPlugin(const FString& PluginName, const FString& NameToReplace, const FString& PluginLocation, const FNewPluginParamsWithDescriptor& CreationParams, FLoadPluginParams& LoadParams);
	
	/**
	 * Helper to create and load a new plugin
	 * @param PluginFilePath Plugin descriptor file path
	 * @param CreationParams Plugin creation parameters
	 * @param LoadParams Plugin loading parameters
	 * @return The newly created plugin. If something goes wrong during the creation process, the plugin folder gets deleted and null is returned.
	 * @note LoadParams.OutFailReason outputs the reason the plugin creation or loading failed (if applicable)
	 * @note Will fail if the plugin already exists
	 * @note This function can circumvent the standard plugin directory structure, in which case the newly created plugin may also need to be loaded by full path
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> CreateAndLoadNewPlugin(const FString& PluginFilePath, const FNewPluginParams& CreationParams, FLoadPluginParams& LoadParams);

	/**
	 * Helper to create and load a new plugin
	 * @param PluginFilePath Plugin descriptor file path
	 * @param CreationParams Plugin creation parameters
	 * @param LoadParams Plugin loading parameters
	 * @return The newly created plugin. If something goes wrong during the creation process, the plugin folder gets deleted and null is returned.
	 * @note LoadParams.OutFailReason outputs the reason the plugin creation or loading failed (if applicable)
	 * @note Will fail if the plugin already exists
	 * @note This function can circumvent the standard plugin directory structure, in which case the newly created plugin may also need to be loaded by full path
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> CreateAndLoadNewPlugin(const FString& PluginFilePath, const FNewPluginParamsWithDescriptor& CreationParams, FLoadPluginParams& LoadParams);

	/**
	 * Load/mount the specified plugin
	 * @param PluginName Plugin name
	 * @param PluginLocation Directory that contains the plugin folder
	 * @param LoadParams Plugin loading parameters
	 * @return The loaded plugin or null on failure
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> LoadPlugin(const FString& PluginName, const FString& PluginLocation, FLoadPluginParams& LoadParams);

	/**
	 * Load/mount the specified plugin
	 * @param PluginName Plugin name
	 * @param PluginLocation Directory that contains the plugin folder
	 * @return The loaded plugin or null on failure
	 */
	static TSharedPtr<IPlugin> LoadPlugin(const FString& PluginName, const FString& PluginLocation)
	{
		FLoadPluginParams Params;
		return LoadPlugin(PluginName, PluginLocation, Params);
	}

	/**
	 * Load/mount the specified plugin
	 * @param PluginFilePath Plugin descriptor file path
	 * @param LoadParams Plugin loading parameters
	 * @return The loaded plugin or null on failure
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> LoadPlugin(const FString& PluginFilePath, FLoadPluginParams& LoadParams);

	/**
	 * Load/mount the specified plugin
	 * @param PluginFilePath Plugin descriptor file path
	 * @return The loaded plugin or null on failure
	 */
	static TSharedPtr<IPlugin> LoadPlugin(const FString& PluginFilePath)
	{
		FLoadPluginParams Params;
		return LoadPlugin(PluginFilePath, Params);
	}

	/**
	 * Finds a loaded plugin from a plugin descriptor file path
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> FindLoadedPlugin(const FString& PluginDescriptorFileName);

	/**
	 * Finds the plugin that contains the specified package, if any
	 * @note This method supports C++ module packages (with the "/Script/" root)
	 * @param PackagePath Unreal path of the package
	 * @return Plugin or null if the package is not in a plugin
	 */
	static PLUGINUTILS_API TSharedPtr<IPlugin> FindPluginFromPackagePath(FName PackagePath);
	static PLUGINUTILS_API TSharedPtr<IPlugin> FindPluginFromPackagePath(FStringView PackagePath);

	/**
	 * Unload assets from the specified plugin and unmount it
	 * @note Only works on content-only plugins; plugins with code modules cannot be safely unloaded
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param Plugin Plugin to unload
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether the plugin was successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPlugin(const TSharedRef<IPlugin>& Plugin, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugin and unmount it
	 * @note Only works on content-only plugins; plugins with code modules cannot be safely unloaded
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param PluginName Name of the plugin to unload
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether the plugin was successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPlugin(const FString& PluginName, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugins and unmount them
	 * @note Only works on content-only plugins; plugins with code modules cannot be safely unloaded
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param Plugins Plugins to unload
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether all plugins were successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPlugins(const TConstArrayView<TSharedRef<IPlugin>> Plugins, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugins and unmount them
	 * @note Only works on content-only plugins; plugins with code modules cannot be safely unloaded
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param PluginNames Names of the plugins to unload
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether all plugins were successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPlugins(const TConstArrayView<FString> PluginNames, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugin but does not unmount it
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param Plugin Plugin to unload assets from
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether plugin assets were successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPluginAssets(const TSharedRef<IPlugin>& Plugin, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugin but does not unmount it
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param PluginName Name of the plugin to unload assets from
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether plugin assets were successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPluginAssets(const FString& PluginName, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugins but does not unmount them
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param Plugin Plugins to unload assets from
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether plugin assets were successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPluginsAssets(const TConstArrayView<TSharedRef<IPlugin>> Plugins, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugin but does not unmount them
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param PluginNames Names of the plugins to unload assets from
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether plugin assets were successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPluginsAssets(const TSet<FString>& PluginNames, FText* OutFailReason = nullptr);

	/**
	 * Unload assets from the specified plugin but does not unmount them
	 * @warning Dirty assets that need to be saved will be unloaded anyway
	 * @param PluginNames Names of the plugins to unload assets from
	 * @param OutFailReason Outputs the reason of the failure if any
	 * @return Whether plugin assets were successfully unloaded
	 */
	static PLUGINUTILS_API bool UnloadPluginsAssets(const TConstArrayView<FString> PluginNames, FText* OutFailReason = nullptr);

	/**
	 * Adds a directory to the list of paths that are recursively searched for plugins, 
	 * if that directory isn't already under the search paths.
	 * @param Dir Directory to add (doesn't have to be an absolute or normalized path)
	 * @param bRefreshPlugins Whether to refresh plugins if the search path list gets modified
	 * @param bUpdateProjectFile Whether to update the project additional plugin directories (persistently saved in uproject file) if needed
	 * @return Whether the plugin search path was modified
	 */
	static PLUGINUTILS_API bool AddToPluginSearchPathIfNeeded(const FString& Dir, bool bRefreshPlugins = false, bool bUpdateProjectFile = false);

	/**
	 * Validate that the plugin name is valid, that the name isn't already used by a registered plugin
	 * and optionally that there isn't an unregistered plugin with that name that exists at the specified location.
	 * @param PluginName Plugin name
	 * @param PluginLocation Optional directory in which to look for a plugin that might not be registered
	 * @param FailReason Optional output text describing why the validation failed
	 * @return
	 */
	static PLUGINUTILS_API bool ValidateNewPluginNameAndLocation(const FString& PluginName, const FString& PluginLocation = FString(), FText* FailReason = nullptr);

	/**
	 * Validate that the plugin name is valid, that the name isn't already used by a registered plugin
	 * and optionally that there isn't an unregistered plugin with that name that exists at the plugin folder
	 * @param PluginFilePath Plugin descriptor file path
	 * @param FailReason Optional output text describing why the validation failed
	 * @return
	 */
	static PLUGINUTILS_API bool ValidateNewPluginFilePath(const FString& PluginFilePath, FText* FailReason = nullptr);

	/**
	 * Returns whether the specified plugin name is valid, regardless of whether it's already used
	 * @param PluginName Plugin name
	 * @param FailReason Optional output text specifying what is wrong with the plugin name
	 * @param PluginTermReplacement If set, replaces the term "plugin" in the fail reason message
	 */
	static PLUGINUTILS_API bool IsValidPluginName(const FString& PluginName, FText* FailReason = nullptr, const FText* PluginTermReplacement = nullptr);
};

#endif //if WITH_EDITOR
